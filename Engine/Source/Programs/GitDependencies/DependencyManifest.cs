// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Xml.Serialization;
using System.ComponentModel;

namespace GitDependencies
{
	/// <summary>
	/// Contains information about a binary dependency in the workspace
	/// </summary>
	[DebuggerDisplay("{Name}")]
	public class DependencyFile
	{
		/// <summary>
		/// Name of the file, relative to the root. Directories are separated with forward slashes, without a leading slash.
		/// </summary>
		[XmlAttribute]
		public string Name = null!;

		/// <summary>
		/// Hash of this file. Used to determine which blob represents its contents.
		/// </summary>
		[XmlAttribute]
		public string Hash = null!;

		/// <summary>
		/// Whether this file should have the exectuable bit set, on platforms that support it.
		/// </summary>
		[XmlAttribute]
		[DefaultValue(false)]
		public bool IsExecutable;
	}

	/// <summary>
	/// Represents a hashed binary blob of data.
	/// </summary>
	[DebuggerDisplay("{Hash}")]
	public class DependencyBlob
	{
		/// <summary>
		/// Hash of the blob.
		/// </summary>
		[XmlAttribute]
		public string Hash = null!;

		/// <summary>
		/// Size of the blob.
		/// </summary>
		[XmlAttribute]
		public long Size;

		/// <summary>
		/// Hash of the pack file containing this blob.
		/// </summary>
		[XmlAttribute]
		public string PackHash = null!;

		/// <summary>
		/// Offset of this blob's data within the pack file.
		/// </summary>
		[XmlAttribute]
		public long PackOffset;
	}

	/// <summary>
	/// A downloadable data file which can contain several blobs.
	/// </summary>
	[DebuggerDisplay("{Hash}")]
	public class DependencyPack
	{
		/// <summary>
		/// Hash of this pack file.
		/// </summary>
		[XmlAttribute]
		public string Hash = null!;

		/// <summary>
		/// Size of this pack file, when uncompressed.
		/// </summary>
		[XmlAttribute]
		public long Size;

		/// <summary>
		/// Compressed size of this pack file.
		/// </summary>
		[XmlAttribute]
		public long CompressedSize;

		/// <summary>
		/// Subdirectory for this pack file on the remote server.
		/// </summary>
		[XmlAttribute]
		public string RemotePath = null!;
	}

	/// <summary>
	/// A manifest for dependencies that should be unpacked into the workspace at a given revision.
	/// </summary>
	public class DependencyManifest
	{
		/// <summary>
		/// Base URL for downloading pack files from.
		/// </summary>
		[XmlAttribute]
		public string BaseUrl = "http://cdn.unrealengine.com/dependencies";

		/// <summary>
		/// Whether to ignore proxy servers when downloading files from this remote.
		/// </summary>
		[XmlAttribute]
		[DefaultValue(false)]
		public bool IgnoreProxy;

		/// <summary>
		/// Files which should be extracted into the workspace.
		/// </summary>
		[XmlArrayItem("File")]
		public DependencyFile[] Files = new DependencyFile[0];

		/// <summary>
		/// All the blobs required for entries in the Files array.
		/// </summary>
		[XmlArrayItem("Blob")]
		public DependencyBlob[] Blobs = new DependencyBlob[0];

		/// <summary>
		/// All the packs required for entries in the Blobs array
		/// </summary>
		[XmlArrayItem("Pack")]
		public DependencyPack[] Packs = new DependencyPack[0];
	}
}
