// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;

namespace GitDependencies
{
	class ForkReadStream : Stream
	{
		Stream? Inner;
		Stream? ForkWrite;

		public ForkReadStream(Stream? InInner, Stream? InForkWrite)
		{
			Inner = InInner;
			ForkWrite = InForkWrite;
		}

		protected override void Dispose(bool Disposing)
		{
			if (ForkWrite != null)
			{
				ForkWrite.Dispose();
				ForkWrite = null;
			}
			if (Inner != null)
			{
				Inner.Dispose();
				Inner = null;
			}
		}

		public override bool CanRead
		{
			get { return true; }
		}

		public override bool CanWrite
		{
			get { return false; }
		}

		public override bool CanSeek
		{
			get { return false; }
		}

		public override long Position
		{
			get
			{
				return Inner!.Position;
			}
			set
			{
				throw new NotImplementedException();
			}
		}

		public override long Length
		{
			get { return Inner!.Length; }
		}

		public override void SetLength(long Value)
		{
			throw new NotImplementedException();
		}

		public override int Read(byte[] Buffer, int Offset, int Count)
		{
			int SizeRead = Inner!.Read(Buffer, Offset, Count);
			if (SizeRead > 0)
			{
				ForkWrite!.Write(Buffer, Offset, SizeRead);
			}
			return SizeRead;
		}

		public override void Write(byte[] Buffer, int Offset, int Count)
		{
			throw new NotImplementedException();
		}

		public override long Seek(long Offset, SeekOrigin Origin)
		{
			throw new NotImplementedException();
		}

		public override void Flush()
		{
			ForkWrite!.Flush();
		}
	}
}
