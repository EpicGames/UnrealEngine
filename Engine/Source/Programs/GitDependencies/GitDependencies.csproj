<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
    <Configuration Condition=" '$(Configuration)' == '' ">Development</Configuration>
    <OutputType>Exe</OutputType>
	<PublishTrimmed>false</PublishTrimmed>
    <AppendTargetFrameworkToOutputPath>false</AppendTargetFrameworkToOutputPath>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <Configurations>Debug;Release;Development</Configurations>
    <AppendRuntimeIdentifierToOutputPath>false</AppendRuntimeIdentifierToOutputPath>
    <DebugType>pdbonly</DebugType> 
    <Nullable>enable</Nullable> 
    <PlatformTarget>AnyCPU</PlatformTarget> 
    <StartupObject>GitDependencies.Program</StartupObject>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Development|AnyCPU' ">
    <Optimize>True</Optimize>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|AnyCPU' ">
    <Optimize>True</Optimize>
  </PropertyGroup>
</Project>