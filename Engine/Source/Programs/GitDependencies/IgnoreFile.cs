// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Text.RegularExpressions;

namespace GitDependencies
{
	class IgnoreFile
	{
		List<Tuple<Regex, bool>> Patterns = new List<Tuple<Regex, bool>>();

		public IgnoreFile(string FileName) : this(File.ReadAllLines(FileName))
		{
		}

		public IgnoreFile(string[] Lines)
		{
			foreach(string Line in Lines)
			{
				string TrimLine = Line.Trim();
				if(TrimLine.Length > 0 && !TrimLine.StartsWith("#"))
				{
					// Strip the '!' character if it's an include rule
					bool bExcludeFile = true;
					if(TrimLine.StartsWith("!"))
					{
						TrimLine = TrimLine.Substring(1).TrimStart();
						bExcludeFile = false;
					}

					// Convert the rest of the line into a Regex
					string FinalPattern = "^" + Regex.Escape(TrimLine.Replace('\\', '/')) + "$";
					FinalPattern = FinalPattern.Replace("\\?", ".");
					FinalPattern = FinalPattern.Replace("\\*\\*", ".*");
					FinalPattern = FinalPattern.Replace("\\*", "[^/]*");

					// Fixup partial matches (patterns that aren't rooted, or match an entire directory)
					if(!FinalPattern.StartsWith("^/"))
					{
						FinalPattern = FinalPattern.Substring(1);
					}
					if(FinalPattern.EndsWith("/$"))
					{
						FinalPattern = FinalPattern.Substring(0, FinalPattern.Length - 1);
					}

					// Add it to the list of patterns
					Patterns.Add(new Tuple<Regex,bool>(new Regex(FinalPattern, RegexOptions.IgnoreCase), bExcludeFile));
				}
			}
		}

		public bool IsExcludedFile(string FilePath)
		{
			string NormalizedFilePath = FilePath.Replace('\\', '/');
			if(!NormalizedFilePath.StartsWith("/"))
			{
				NormalizedFilePath = "/" + NormalizedFilePath;
			}

			bool bIsExcluded = false;
			foreach (Tuple<Regex, bool> Pattern in Patterns)
			{
				if (bIsExcluded != Pattern.Item2 && Pattern.Item1.IsMatch(NormalizedFilePath))
				{
					bIsExcluded = Pattern.Item2;
				}
			}
			return bIsExcluded;
		}
	}
}
