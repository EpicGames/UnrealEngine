// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace GitDependencies
{
	static class Log
	{
		static string CurrentStatus = "";

		public static void WriteLine()
		{
			FlushStatus();
			Console.WriteLine();
		}

		public static void WriteLine(string Line)
		{
			FlushStatus();
			Console.WriteLine(Line);
		}

		public static void WriteLine(string Format, params object[] Args)
		{
			FlushStatus();
			Console.WriteLine(Format, Args);
		}

		public static void WriteError(string Format, params object[] Args)
		{
			FlushStatus();
			Console.ForegroundColor = ConsoleColor.Red;
			Console.WriteLine(Format, Args);
			Console.ResetColor();
		}

		public static void WriteStatus(string Status)
		{
			// If the status is larger than the console width, truncate it
			string NewStatus = Status;
			try
			{
				int Width = Console.BufferWidth;
				if(NewStatus.Length >= Width)
				{
					NewStatus = NewStatus.Substring(0, Width - 1);
				}
			}
			catch(Exception)
			{
			}

			// Write the new status, and clear any space after the end of the string if it's shorter
			Console.Write("\r" + NewStatus);
			if(NewStatus.Length < CurrentStatus.Length)
			{
				Console.Write(new string(' ', CurrentStatus.Length - NewStatus.Length) + "\r" + NewStatus);
			}
			CurrentStatus = NewStatus;
		}

		public static void WriteStatus(string Format, params object[] Args)
		{
			WriteStatus(String.Format(Format, Args));
		}

		public static void FlushStatus()
		{
			if(CurrentStatus.Length > 0)
			{
				Console.WriteLine();
				CurrentStatus = "";
			}
		}
	}
}
