// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;

namespace GitDependencies
{
	delegate void NotifyReadDelegate(int SizeRead);

	class NotifyReadStream : Stream
	{
		Stream? Inner;
		NotifyReadDelegate NotifyRead;
		long TotalRead;

		public NotifyReadStream(Stream? InInner, NotifyReadDelegate InNotifyRead)
		{
			Inner = InInner;
			NotifyRead = InNotifyRead;
			TotalRead = 0;
		}

		protected override void Dispose(bool Disposing)
		{
			if(Inner != null)
			{
				Inner.Dispose();
				Inner = null;
			}
		}

		public override bool CanRead
		{
			get { return true; }
		}

		public override bool CanWrite
		{
			get { return false; }
		}

		public override bool CanSeek
		{
			get { return false; }
		}

		public override long Position
		{
			get
			{
				return TotalRead;
			}
			set
			{
				throw new NotImplementedException();
			}
		}

		public override long Length
		{
			get { return Inner!.Length; }
		}

		public override void SetLength(long Value)
		{
			throw new NotImplementedException();
		}

		public override int Read(byte[] Buffer, int Offset, int Count)
		{
			int SizeRead = Inner!.Read(Buffer, Offset, Count);
			NotifyRead(SizeRead);
			TotalRead += SizeRead;
			return SizeRead;
		}

		public override void Write(byte[] Buffer, int Offset, int Count)
		{
			throw new NotImplementedException();
		}

		public override long Seek(long Offset, SeekOrigin Origin)
		{
			throw new NotImplementedException();
		}

		public override void Flush()
		{
			Inner!.Flush();
		}
	}
}
