// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Security.Cryptography;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using System.Xml;
using System.Xml.Serialization;

namespace GitDependencies
{
	class Program
	{
		class AsyncDownloadState
		{
			public int NumFiles;
			public int NumFilesRead;
			public long NumBytesRead;
			public long NumBytesTotal;
			public long NumBytesCached;
			public int NumFailingOrIdleDownloads;
			public string LastDownloadError = null!;
		}

		enum OverwriteMode
		{
			Unchanged,
			Prompt,
			Force,
		}

		class IncomingPack
		{
			public string Url = null!;
			public string Hash = null!;
			public string? CacheFileName = null!;
			public IncomingFile[] Files = new IncomingFile[0];
			public long CompressedSize;
		}

		class IncomingFile
		{
			public string[] Names = new string[0];
			public string Hash = null!;
			public long MinPackOffset;
			public long MaxPackOffset;
		}

		struct DependencyPackInfo
		{
			public DependencyManifest Manifest;
			public DependencyPack Pack;

			public DependencyPackInfo(DependencyManifest Manifest, DependencyPack Pack) 
			{
				this.Manifest = Manifest;
				this.Pack = Pack;
			}

			public string GetCacheFileName() 
			{
				return Path.Combine(Pack.Hash.Substring(0, 2), Pack.Hash);
			}
		}

		class CorruptPackFileException : Exception
		{
			public CorruptPackFileException(string Message, Exception? InnerException)
				: base(Message, InnerException)
			{
			}
		}

		const string IncomingFileSuffix = ".incoming";
		const string TempManifestExtension = ".tmp";

		const string ManifestFilename = ".uedependencies";
		const string LegacyManifestFilename = ".ue4dependencies";

		static readonly string InstanceSuffix = Guid.NewGuid().ToString().Replace("-", "");
		static HttpClient? HttpClientInstance = null;

		static int Main(string[] Args)
		{
			// Build the argument list. Remove any double-hyphens from the start of arguments for conformity with other Epic tools.
			List<string> ArgsList = new List<string>(Args);
			NormalizeArguments(ArgsList);

			// Find the default arguments from the UE_GITDEPS_ARGS environment variable. These arguments do not cause an error if duplicated or redundant, but can still override defaults.
			List<string> DefaultArgsList = SplitArguments(Environment.GetEnvironmentVariable("UE_GITDEPS_ARGS"));
			NormalizeArguments(DefaultArgsList);

			// Parse the parameters
			int NumThreads = ParseIntParameter(ArgsList, DefaultArgsList, "-threads=", Math.Max(4, (int)(Environment.ProcessorCount * .25)));
			int MaxRetries = ParseIntParameter(ArgsList, DefaultArgsList, "-max-retries=", 4);
			bool bDryRun = ParseSwitch(ArgsList, "-dry-run");
			bool bHelp = ParseSwitch(ArgsList, "-help");
			float CacheSizeMultiplier = ParseFloatParameter(ArgsList, DefaultArgsList, "-cache-size-multiplier=", 2.0f);
			int CacheDays = ParseIntParameter(ArgsList, DefaultArgsList, "-cache-days=", 7);
			string RootPath = ParseParameter(ArgsList, "-root=", Path.GetFullPath(Path.Combine(Path.GetDirectoryName(Assembly.GetExecutingAssembly()!.Location!)!, "../../../../..")))!;
			double HttpTimeoutMultiplier = ParseFloatParameter(ArgsList, DefaultArgsList, "-http-timeout-multiplier=", 1.0f) * NumThreads;

			// Parse the cache path. A specific path can be set using -catch=<PATH> or the UE4_GITDEPS environment variable, otherwise we look for a parent .git directory
			// and use a sub-folder of that. Users which download the source through a zip file (and won't have a .git directory) are unlikely to benefit from caching, as
			// they only need to download dependencies once.
			string? CachePath = null;
			if (!ParseSwitch(ArgsList, "-no-cache"))
			{
				string? CachePathParam = ParseParameter(ArgsList, DefaultArgsList, "-cache=", Environment.GetEnvironmentVariable("UE_GITDEPS"));
				if (String.IsNullOrEmpty(CachePathParam))
				{
					string? CheckPath = Path.GetFullPath(RootPath);
					while (CheckPath != null)
					{
						string GitPath = Path.Combine(CheckPath, ".git");
						if (Directory.Exists(GitPath))
						{
							CachePath = Path.Combine(GitPath, "ue-gitdeps");
							if (!Directory.Exists(CachePath))
							{
								// Migrate the ue4 path to ue.
								string UE4CachePath = Path.Combine(GitPath, "ue4-gitdeps");
								if (Directory.Exists(UE4CachePath))
								{
									Directory.Move(UE4CachePath, CachePath);
								}
							}
							break;
						}
						CheckPath = Path.GetDirectoryName(CheckPath);
					}
				}
				else
				{
					CachePath = Path.GetFullPath(CachePathParam);
				}
			}

			// Parse the overwrite mode
			OverwriteMode Overwrite = OverwriteMode.Unchanged;
			if(ParseSwitch(ArgsList, "-prompt"))
			{
				Overwrite = OverwriteMode.Prompt;
			}
			else if(ParseSwitch(ArgsList, "-force"))
			{
				Overwrite = OverwriteMode.Force;
			}

			// Setup network proxy from argument list or environment variable
			string? ProxyUrl = ParseParameter(ArgsList, DefaultArgsList, "-proxy=", null);
			if(String.IsNullOrEmpty(ProxyUrl))
			{
				ProxyUrl = Environment.GetEnvironmentVariable("HTTP_PROXY");
				if(String.IsNullOrEmpty(ProxyUrl))
				{
					ProxyUrl = Environment.GetEnvironmentVariable("http_proxy");
				}
			}

			// Create a URI for the proxy. If there's no included username/password, accept them as separate parameters for legacy reasons.
			Uri? Proxy = null;
			if(!String.IsNullOrEmpty(ProxyUrl))
			{
				UriBuilder ProxyBuilder = new UriBuilder(ProxyUrl);
				if(String.IsNullOrEmpty(ProxyBuilder.UserName))
				{
					ProxyBuilder.UserName = ParseParameter(ArgsList, DefaultArgsList, "-proxy-user=", null);
				}
				if(String.IsNullOrEmpty(ProxyBuilder.Password))
				{
					ProxyBuilder.Password = ParseParameter(ArgsList, DefaultArgsList, "-proxy-password=", null);
				}
				Proxy = ProxyBuilder.Uri;
			}

			// Parse all the default exclude filters
			HashSet<string> ExcludeFolders = new HashSet<string>(StringComparer.CurrentCultureIgnoreCase);
			foreach(string ExcludeFolder in ParseParameters(ArgsList, "-exclude="))
			{
				ExcludeFolders.Add(ExcludeFolder.Replace('\\', '/').TrimEnd('/'));
			}

			// If there are any more parameters, print an error
			foreach(string RemainingArg in ArgsList)
			{
				Log.WriteLine("Invalid command line parameter: {0}", RemainingArg);
				Log.WriteLine();
				bHelp = true;
			}

			// Print the help message
			if(bHelp)
			{
				Log.WriteLine("Usage:");
				Log.WriteLine("   GitDependencies [options]");
				Log.WriteLine();
				Log.WriteLine("Options:");
				Log.WriteLine("   --all                         Sync all folders");
				Log.WriteLine("   --include=<X>                 Include binaries in folders called <X>");
				Log.WriteLine("   --exclude=<X>                 Exclude binaries in folders called <X>");
				Log.WriteLine("   --prompt                      Prompt before overwriting modified files");
				Log.WriteLine("   --force                       Always overwrite modified files");
				Log.WriteLine("   --root=<PATH>                 Set the repository directory to be sync");
				Log.WriteLine("   --threads=<N>                 Use N threads when downloading new files");
				Log.WriteLine("   --dry-run                     Print a list of outdated files and exit");
				Log.WriteLine("   --http-timeout-multiplier=<N> Override download timeout multiplier");
				Log.WriteLine("   --max-retries                 Override maximum number of retries per file");
				Log.WriteLine("   --proxy=<user:password@url>   Sets the HTTP proxy address and credentials");
				Log.WriteLine("   --cache=<PATH>                Specifies a custom path for the download cache");
				Log.WriteLine("   --cache-size-multiplier=<N>   Cache size as multiplier of current download");
				Log.WriteLine("   --cache-days=<N>              Number of days to keep entries in the cache");
				Log.WriteLine("   --no-cache                    Disable caching of downloaded files");
				Log.WriteLine();
				Log.WriteLine("Detected settings:");
				Log.WriteLine("   Excluded folders: {0}", (ExcludeFolders.Count == 0)? "none" : String.Join(", ", ExcludeFolders));
				Log.WriteLine("   Proxy server: {0}", (Proxy == null)? "none" : Proxy.ToString());
				Log.WriteLine("   Download cache: {0}", (CachePath == null)? "disabled" : CachePath);
				Log.WriteLine();
				Log.WriteLine("Default arguments can be set through the UE_GITDEPS_ARGS environment variable.");
				return 0;
			}

			// Register a delegate to clear the status text if we use ctrl-c to quit
			Console.CancelKeyPress += delegate { Log.FlushStatus(); };

			// Update the tree. Make sure we clear out the status line if we quit for any reason (eg. ctrl-c)
			if(!UpdateWorkingTree(bDryRun, RootPath, ExcludeFolders, NumThreads, HttpTimeoutMultiplier, MaxRetries, Proxy, Overwrite, CachePath!, CacheSizeMultiplier, CacheDays))
			{
				return 1;
			}
			return 0;
		}

		static void NormalizeArguments(List<string> ArgsList)
		{
			for(int Idx = 0; Idx < ArgsList.Count; Idx++)
			{
				if(ArgsList[Idx].StartsWith("--"))
				{
					ArgsList[Idx] = ArgsList[Idx].Substring(1);
				}
			}
		}

		static List<string> SplitArguments(string? Text)
		{
			List<string> ArgsList = new List<string>();
			if(!String.IsNullOrEmpty(Text))
			{
				for(int Idx = 0; Idx < Text.Length; Idx++)
				{
					if(!Char.IsWhiteSpace(Text[Idx]))
					{
						StringBuilder Arg = new StringBuilder();
						for(bool bInQuotes = false; Idx < Text.Length; Idx++)
						{
							if(!bInQuotes && Char.IsWhiteSpace(Text[Idx]))
							{
								break;
							}
							else if(Text[Idx] == '\"')
							{
								bInQuotes ^= true;
							}
							else
							{
								Arg.Append(Text[Idx]);
							}
						}
						ArgsList.Add(Arg.ToString());
					}
				}
			}
			return ArgsList;
		}

		static bool ParseSwitch(List<string> ArgsList, string Name)
		{
			for(int Idx = 0; Idx < ArgsList.Count; Idx++)
			{
				if(String.Compare(ArgsList[Idx], Name, true) == 0)
				{
					ArgsList.RemoveAt(Idx);
					return true;
				}
			}
			return false;
		}

		static string? ParseParameter(List<string> ArgsList, string Prefix, string? Default)
		{
			string? Value = Default;
			for(int Idx = 0; Idx < ArgsList.Count; Idx++)
			{
				if(ArgsList[Idx].StartsWith(Prefix, StringComparison.CurrentCultureIgnoreCase))
				{
					Value = ArgsList[Idx].Substring(Prefix.Length);
					ArgsList.RemoveAt(Idx);
					break;
				}
			}
			return Value;
		}

		static string? ParseParameter(List<string> ArgsList, List<string> DefaultArgsList, string Prefix, string? Default)
		{
			return ParseParameter(ArgsList, Prefix, ParseParameter(DefaultArgsList, Prefix, Default));
		}

		static int ParseIntParameter(List<string> ArgsList, string Prefix, int Default)
		{
			for(int Idx = 0; Idx < ArgsList.Count; Idx++)
			{
				int Value;
				if(ArgsList[Idx].StartsWith(Prefix, StringComparison.CurrentCultureIgnoreCase) && int.TryParse(ArgsList[Idx].Substring(Prefix.Length), out Value))
				{
					ArgsList.RemoveAt(Idx);
					return Value;
				}
			}
			return Default;
		}

		static int ParseIntParameter(List<string> ArgsList, List<string> DefaultArgsList, string Prefix, int Default)
		{
			return ParseIntParameter(ArgsList, Prefix, ParseIntParameter(DefaultArgsList, Prefix, Default));
		}

		static float ParseFloatParameter(List<string> ArgsList, string Prefix, float Default)
		{
			for(int Idx = 0; Idx < ArgsList.Count; Idx++)
			{
				float Value;
				if(ArgsList[Idx].StartsWith(Prefix, StringComparison.CurrentCultureIgnoreCase) && float.TryParse(ArgsList[Idx].Substring(Prefix.Length), out Value))
				{
					ArgsList.RemoveAt(Idx);
					return Value;
				}
			}
			return Default;
		}

		static float ParseFloatParameter(List<string> ArgsList, List<string> DefaultArgsList, string Prefix, float Default)
		{
			return ParseFloatParameter(ArgsList, Prefix, ParseFloatParameter(DefaultArgsList, Prefix, Default));
		}

		static IEnumerable<string> ParseParameters(List<string> ArgsList, string Prefix)
		{
			for(;;)
			{
				string? Value = ParseParameter(ArgsList, Prefix, null);
				if(Value == null)
				{
					break;
				}
				yield return Value;
			}
		}

		static bool UpdateWorkingTree(bool bDryRun, string RootPath, HashSet<string> ExcludeFolders, int NumThreads, double HttpTimeoutMultiplier, int MaxRetries, Uri? Proxy, OverwriteMode Overwrite, string CachePath, float CacheSizeMultiplier, int CacheDays)
		{
			// Start scanning on the working directory 
			if(ExcludeFolders.Count > 0)
			{
				Log.WriteLine("Checking dependencies (excluding {0})...", String.Join(", ", ExcludeFolders));
			}
			else
			{
				Log.WriteLine("Checking dependencies...");
			}

			// Read the .gitdepsignore file, if there is one
			IgnoreFile? IgnoreFile = null;
			try
			{
				string IgnoreFileName = Path.Combine(RootPath, ".gitdepsignore");
				if(File.Exists(IgnoreFileName))
				{
					IgnoreFile = new IgnoreFile(IgnoreFileName);
				}
			}
			catch
			{
				Log.WriteLine("Failed to read .gitdepsignore file.");
				return false;
			}

			// Figure out the path to the working manifest
			string WorkingManifestPath = Path.Combine(RootPath, ManifestFilename);
			if (!File.Exists(WorkingManifestPath))
			{
				string LegacyManifestPath = Path.Combine(RootPath, LegacyManifestFilename);
				if (File.Exists(LegacyManifestPath) || File.Exists(LegacyManifestPath + TempManifestExtension))
				{
					WorkingManifestPath = LegacyManifestPath;
				}
			}

			// Recover from any interrupted transaction to the working manifest, by moving the temporary file into place.
			string TempWorkingManifestPath = WorkingManifestPath + TempManifestExtension;
			if(File.Exists(TempWorkingManifestPath) && !File.Exists(WorkingManifestPath) && !SafeMoveFile(TempWorkingManifestPath, WorkingManifestPath))
			{
				return false;
			}

			// Read the initial manifest, or create a new one
			WorkingManifest CurrentManifest;
			if(!File.Exists(WorkingManifestPath) || !ReadXmlObject(WorkingManifestPath, out CurrentManifest))
			{
				CurrentManifest = new WorkingManifest();
			}

			// Remove all the in-progress download files left over from previous runs
			foreach(WorkingFile InitialFile in CurrentManifest.Files)
			{
				if(InitialFile.Timestamp == 0)
				{
					string IncomingFilePath = Path.Combine(RootPath, InitialFile.Name + IncomingFileSuffix);
					if(File.Exists(IncomingFilePath) && !SafeDeleteFile(IncomingFilePath))
					{
						return false;
					}
				}
			}

			// Find all the manifests and push them into dictionaries
			Dictionary<string, DependencyFile> TargetFiles = new Dictionary<string,DependencyFile>(StringComparer.InvariantCultureIgnoreCase);
			Dictionary<string, DependencyBlob> TargetBlobs = new Dictionary<string,DependencyBlob>(StringComparer.InvariantCultureIgnoreCase);
			Dictionary<string, DependencyPackInfo> TargetPacks = new Dictionary<string, DependencyPackInfo>(StringComparer.InvariantCultureIgnoreCase);
			foreach(string BaseFolder in Directory.EnumerateDirectories(RootPath))
			{
				if (!AddManifests(TargetFiles, TargetBlobs, TargetPacks, Path.Combine(BaseFolder, "Build"), ""))
				{
					return false;
				}

				if (!AddPluginManifests(TargetFiles, TargetBlobs, TargetPacks, Path.Combine(BaseFolder, "Plugins"), Path.GetFileName(BaseFolder) + "/Plugins"))
				{
					return false;
				}
			}

			// Find all the existing files in the working directory from previous runs. Use the working manifest to cache hashes for them based on timestamp, but recalculate them as needed.
			List<WorkingFile> ReadOnlyFiles = new List<WorkingFile>();
			Dictionary<string, WorkingFile> CurrentFileLookup = new Dictionary<string, WorkingFile>();
			foreach(WorkingFile CurrentFile in CurrentManifest.Files)
			{
				// Update the hash for this file
				string CurrentFilePath = Path.Combine(RootPath, CurrentFile.Name);
				FileInfo CurrentFileInfo = new FileInfo(CurrentFilePath);
				if(CurrentFileInfo.Exists)
				{
					long LastWriteTime = CurrentFileInfo.LastWriteTimeUtc.Ticks;
					if(LastWriteTime != CurrentFile.Timestamp)
					{
						CurrentFile.Hash = ComputeHashForFile(CurrentFilePath);
						CurrentFile.Timestamp = LastWriteTime;
					}
					CurrentFileLookup.Add(CurrentFile.Name, CurrentFile);

					if (CurrentFileInfo.IsReadOnly)
					{
						ReadOnlyFiles.Add(CurrentFile);
					}
				}
			}

			// Also add all the untracked files which already exist, but weren't downloaded by this program
			foreach (DependencyFile TargetFile in TargetFiles.Values) 
			{
				if (!CurrentFileLookup.ContainsKey(TargetFile.Name))
				{
					string CurrentFilePath = Path.Combine(RootPath, TargetFile.Name);
					FileInfo CurrentFileInfo = new FileInfo(CurrentFilePath);
					if (CurrentFileInfo.Exists)
					{
						WorkingFile CurrentFile = new WorkingFile
						{
							Name = TargetFile.Name,
							Hash = ComputeHashForFile(CurrentFilePath),
							Timestamp = CurrentFileInfo.LastWriteTimeUtc.Ticks
						};
						CurrentFileLookup.Add(CurrentFile.Name, CurrentFile);

						if (CurrentFileInfo.IsReadOnly)
						{
							ReadOnlyFiles.Add(CurrentFile);
						}
					}
				}
			}

			// Build a list of all the filtered target files
			List<DependencyFile> FilteredTargetFiles = new List<DependencyFile>();
			foreach(DependencyFile TargetFile in TargetFiles.Values)
			{
				if(!IsExcludedFolder(TargetFile.Name, ExcludeFolders) && (IgnoreFile == null || !IgnoreFile.IsExcludedFile(TargetFile.Name)))
				{
					FilteredTargetFiles.Add(TargetFile);
				}
			}

			// Create a list of files which need to be updated, and a list of the executable files in the 
			List<DependencyFile> FilesToDownload = new List<DependencyFile>();

			// Create a new working manifest for the working directory, moving over files that we already have. Add any missing dependencies into the download queue.
			WorkingManifest NewWorkingManifest = new WorkingManifest();
			foreach (DependencyFile TargetFile in FilteredTargetFiles)
			{
				if(CurrentFileLookup.TryGetValue(TargetFile.Name, out WorkingFile? NewFile) && NewFile.Hash == TargetFile.Hash)
				{
					// Update the expected hash to match what we're looking for
					NewFile.ExpectedHash = TargetFile.Hash;

					// Move the existing file to the new working set
					CurrentFileLookup.Remove(NewFile.Name);
					ReadOnlyFiles.Remove(NewFile);
				}
				else
				{
					// Create a new working file
					NewFile = new WorkingFile
					{
						Name = TargetFile.Name,
						ExpectedHash = TargetFile.Hash
					};

					// Add it to the download list
					FilesToDownload.Add(TargetFile);
				}
				NewWorkingManifest.Files.Add(NewFile);
			}

			// Print out everything that we'd change in a dry run
			if(bDryRun)
			{
				HashSet<string> NewFiles = new HashSet<string>(FilesToDownload.Select(x => x.Name));
				foreach(string RemoveFile in CurrentFileLookup.Keys.Where(x => !NewFiles.Contains(x)))
				{
					Log.WriteLine("Remove {0}", RemoveFile);
				}
				foreach(string UpdateFile in CurrentFileLookup.Keys.Where(x => NewFiles.Contains(x)))
				{
					Log.WriteLine("Update {0}", UpdateFile);
				}
				foreach(string AddFile in NewFiles.Where(x => !CurrentFileLookup.ContainsKey(x)))
				{
					Log.WriteLine("Add {0}", AddFile);
				}
				return true;
			}

			// Delete any files which are no longer needed
			List<WorkingFile> TamperedFiles = new List<WorkingFile>();
			foreach(WorkingFile FileToRemove in CurrentFileLookup.Values)
			{
				if (!IsExcludedFolder(FileToRemove.Name, ExcludeFolders) && (IgnoreFile == null || !IgnoreFile.IsExcludedFile(FileToRemove.Name)))
				{
					if(Overwrite != OverwriteMode.Force && FileToRemove.Hash != FileToRemove.ExpectedHash)
					{
						TamperedFiles.Add(FileToRemove);
					}
					else if(!SafeDeleteFile(Path.Combine(RootPath, FileToRemove.Name)))
					{
						return false;
					}
				}
			}

			// Warn if there were any files that have been tampered with or are read only, and allow the user to choose whether to overwrite them
			bool bOverwriteTamperedFiles = true;
			if (Overwrite != OverwriteMode.Force)
			{
				bool PromptForOverwrite = false;
				if (TamperedFiles.Any())
				{
					PromptForOverwrite = true;
					// List the files that have changed
					Log.WriteError("The following file(s) have been modified:");
					foreach (WorkingFile TamperedFile in TamperedFiles)
					{
						bool readOnly = ReadOnlyFiles.Any(x => string.Equals(x.Name, TamperedFile.Name));
						Log.WriteError("  {0}{1}", TamperedFile.Name, readOnly ? " (read only)" : "");
					}
				}

				// Figure out whether to overwrite the files
				if (PromptForOverwrite)
				{
					if (Overwrite == OverwriteMode.Unchanged)
					{
						Log.WriteError("Re-run with the --force parameter to overwrite them.");
						bOverwriteTamperedFiles = false;
					}
					else
					{
						Log.WriteStatus("Would you like to overwrite your changes (y/n)? ");
						ConsoleKeyInfo KeyInfo = Console.ReadKey(false);
						bOverwriteTamperedFiles = (KeyInfo.KeyChar == 'y' || KeyInfo.KeyChar == 'Y');
						Log.FlushStatus();
					}
				}
			}

			// Overwrite any tampered files, or remove them from the download list
			if (bOverwriteTamperedFiles)
			{
				foreach(WorkingFile TamperedFile in TamperedFiles)
				{
					string FilePath = Path.Combine(RootPath, TamperedFile.Name);
					File.SetAttributes(FilePath, File.GetAttributes(FilePath) & ~FileAttributes.ReadOnly);
					if (!SafeDeleteFile(Path.Combine(RootPath, TamperedFile.Name)))
					{
						return false;
					}
				}
			}
			else
			{
				foreach(WorkingFile FileToIgnore in TamperedFiles.Concat(ReadOnlyFiles))
				{
					if(TargetFiles.TryGetValue(FileToIgnore.Name, out DependencyFile? TargetFile))
					{
						TargetFiles.Remove(FileToIgnore.Name);
						FilesToDownload.Remove(TargetFile);
					}
				}
			}

			// If the file is the old extension, delete and change the file name for future writes.
			if (Path.GetExtension(WorkingManifestPath) == LegacyManifestFilename)
			{
				SafeDeleteFile(WorkingManifestPath);
				WorkingManifestPath = Path.ChangeExtension(WorkingManifestPath, ManifestFilename);
				TempWorkingManifestPath = WorkingManifestPath + TempManifestExtension;
			}

			// Write out the new working manifest, so we can track any files that we're going to download. We always verify missing files on startup, so it's ok that things don't exist yet.
			if(!WriteWorkingManifest(WorkingManifestPath, TempWorkingManifestPath, NewWorkingManifest))
			{
				return false;
			}

			// If there's nothing to do, just print a simpler message and exit early
			if(FilesToDownload.Count > 0)
			{
				// Download all the new dependencies
				if(!DownloadDependencies(RootPath, FilesToDownload, TargetBlobs.Values, TargetPacks.Values, NumThreads, HttpTimeoutMultiplier, MaxRetries, Proxy, CachePath))
				{
					return false;
				}

				// Update all the timestamps and hashes for the output files
				foreach(WorkingFile NewFile in NewWorkingManifest.Files)
				{
					if(NewFile.Hash != NewFile.ExpectedHash)
					{
						string NewFileName = Path.Combine(RootPath, NewFile.Name);
						NewFile.Hash = NewFile.ExpectedHash;
						NewFile.Timestamp = File.GetLastWriteTimeUtc(NewFileName).Ticks;
					}
				}

				// Rewrite the manifest with the results
				if(!WriteWorkingManifest(WorkingManifestPath, TempWorkingManifestPath, NewWorkingManifest))
				{
					return false;
				}

				// Cleanup cache files
				if(CachePath != null)
				{
					PurgeCacheFiles(CachePath, TargetPacks, CacheSizeMultiplier, CacheDays);
				}
			}

			// Update all the executable permissions
			if(!SetExecutablePermissions(RootPath, FilteredTargetFiles))
			{
				return false;
			}

			return true;
		}

		static bool AddPluginManifests(Dictionary<string, DependencyFile> TargetFiles, Dictionary<string, DependencyBlob> TargetBlobs, Dictionary<string, DependencyPackInfo> TargetPacks, string PluginsFolder, string ExtractPrefix)
		{
			if (Directory.Exists(PluginsFolder))
			{
				if (Directory.EnumerateFiles(PluginsFolder, "*.uplugin").GetEnumerator().MoveNext())
				{
					return AddManifests(TargetFiles, TargetBlobs, TargetPacks, Path.Combine(PluginsFolder, "Build"), ExtractPrefix + "/");
				}
				foreach (string Subfolder in Directory.EnumerateDirectories(PluginsFolder))
				{
					string Name = Path.GetFileName(Subfolder);
					if (!Name.StartsWith("."))
					{
						if (!AddPluginManifests(TargetFiles, TargetBlobs, TargetPacks, Subfolder, ExtractPrefix + "/" + Name))
						{
							return false;
						}
					}
				}
			}
			return true;
		}

		static bool AddManifests(Dictionary<string, DependencyFile> TargetFiles, Dictionary<string, DependencyBlob> TargetBlobs, Dictionary<string, DependencyPackInfo> TargetPacks, string BuildFolder, string ExtractPrefix)
		{
			if (Directory.Exists(BuildFolder))
			{
				foreach (string ManifestFileName in Directory.EnumerateFiles(BuildFolder, "*.gitdeps.xml"))
				{
					// Ignore any dotfiles; Mac creates them on non-unix partitions to store permission info.
					if (!Path.GetFileName(ManifestFileName).StartsWith("."))
					{
						// Read this manifest
						DependencyManifest NewTargetManifest;
						if (!ReadXmlObject(ManifestFileName, out NewTargetManifest))
						{
							return false;
						}

						// Add all the files, blobs and packs into the shared dictionaries
						foreach (DependencyFile NewFile in NewTargetManifest.Files)
						{
							NewFile.Name = ExtractPrefix + NewFile.Name;
							TargetFiles[NewFile.Name] = NewFile;
						}
						foreach (DependencyBlob NewBlob in NewTargetManifest.Blobs)
						{
							TargetBlobs[NewBlob.Hash] = NewBlob;
						}
						foreach (DependencyPack NewPack in NewTargetManifest.Packs)
						{
							TargetPacks[NewPack.Hash] = new DependencyPackInfo(NewTargetManifest, NewPack);
						}
					}
				}
			}
			return true;
		}

		static void PurgeCacheFiles(string CachePath, Dictionary<string, DependencyPackInfo> Packs, float CacheSizeMultiplier, int CacheDays)
		{
			// Update the timestamp for all referenced packs
			DateTime CurrentTime = DateTime.UtcNow;
			foreach(DependencyPackInfo Pack in Packs.Values)
			{
				string FileName = Path.Combine(CachePath, Pack.GetCacheFileName());
				if(File.Exists(FileName))
				{
					try { File.SetLastWriteTimeUtc(FileName, CurrentTime); } catch { }
				}
			}

			// Get the size of the cache, and time before which we'll consider deleting entries
			long DesiredCacheSize = (long)(Packs.Values.Sum(x => x.Pack.CompressedSize) * CacheSizeMultiplier);
			DateTime StaleTime = CurrentTime - TimeSpan.FromDays(CacheDays) - TimeSpan.FromSeconds(5); // +5s for filesystems that don't store exact timestamps, like FAT.

			// Enumerate all the files in the cache, and sort them by last write time
			DirectoryInfo CacheDirectory = new DirectoryInfo(CachePath);
			IEnumerable<FileInfo> CacheFiles = CacheDirectory.EnumerateFiles("*", SearchOption.AllDirectories);

			// Find all the files in the cache directory
			long CacheSize = 0;
			foreach(FileInfo StaleFile in CacheFiles.OrderByDescending(x => x.LastWriteTimeUtc))
			{
				if(CacheSize > DesiredCacheSize && StaleFile.LastWriteTimeUtc < StaleTime)
				{
					StaleFile.Delete();
				}
				else
				{
					CacheSize += StaleFile.Length;
				}
			}
		}

		/// <summary>
		/// Gets the file mode on Mac
		/// </summary>
		/// <param name="FileName"></param>
		/// <returns></returns>
		public static int GetFileMode_Mac(string FileName)
		{
			stat64_t stat = new stat64_t();
			int Result = stat64(FileName, stat);
			return (Result >= 0) ? stat.st_mode : -1;
		}

		/// <summary>
		/// Sets the file mode on Mac
		/// </summary>
		/// <param name="FileName"></param>
		/// <param name="Mode"></param>
		public static int SetFileMode_Mac(string FileName, ushort Mode)
		{
			return chmod(FileName, Mode);
		}

		/// <summary>
		/// Gets the file mode on Linux
		/// </summary>
		/// <param name="FileName"></param>
		/// <returns></returns>
		public static int GetFileMode_Linux(string FileName)
		{
			stat64_linux_t stat = new stat64_linux_t();
			int Result = stat64_linux(1, FileName, stat);
			return (Result >= 0) ? (int)stat.st_mode : -1;
		}

		/// <summary>
		/// Sets the file mode on Linux
		/// </summary>
		/// <param name="FileName"></param>
		/// <param name="Mode"></param>
		public static int SetFileMode_Linux(string FileName, ushort Mode)
		{
			return chmod_linux(FileName, Mode);
		}

		#region Mac Native File Methods
#pragma warning disable CS0649
		struct timespec_t
		{
			public ulong tv_sec;
			public ulong tv_nsec;
		}

		[StructLayout(LayoutKind.Sequential)]
		class stat64_t
		{
			public uint st_dev;
			public ushort st_mode;
			public ushort st_nlink;
			public ulong st_ino;
			public uint st_uid;
			public uint st_gid;
			public uint st_rdev;
			public timespec_t st_atimespec;
			public timespec_t st_mtimespec;
			public timespec_t st_ctimespec;
			public timespec_t st_birthtimespec;
			public ulong st_size;
			public ulong st_blocks;
			public uint st_blksize;
			public uint st_flags;
			public uint st_gen;
			public uint st_lspare;
			public ulong st_qspare1;
			public ulong st_qspare2;
		}

		[DllImport("libSystem.dylib")]
		static extern int stat64(string pathname, stat64_t stat);

		[DllImport("libSystem.dylib")]
		static extern int chmod(string path, ushort mode);

#pragma warning restore CS0649
		#endregion

		#region Linux Native File Methods
#pragma warning disable CS0649

		[StructLayout(LayoutKind.Sequential)]
		class stat64_linux_t
		{
			public ulong st_dev;
			public ulong st_ino;
			public ulong st_nlink;
			public uint st_mode;
			public uint st_uid;
			public uint st_gid;
			public int pad0;
			public ulong st_rdev;
			public long st_size;
			public long st_blksize;
			public long st_blocks;
			public timespec_t st_atime;
			public timespec_t st_mtime;
			public timespec_t st_ctime;
			public long glibc_reserved0;
			public long glibc_reserved1;
			public long glibc_reserved2;
		};

		/* stat tends to get compiled to another symbol and libc doesnt directly have that entry point */
		[DllImport("libc", EntryPoint = "__xstat64")]
		static extern int stat64_linux(int ver, string pathname, stat64_linux_t stat);

		[DllImport("libc", EntryPoint = "chmod")]
		static extern int chmod_linux(string path, ushort mode);

#pragma warning restore CS0649
		#endregion

		static bool SetExecutablePermissions(string RootDir, IEnumerable<DependencyFile> Files)
		{
			// This only apply for *NIX and Mac
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return true;
			}

			bool IsLinux = RuntimeInformation.IsOSPlatform(OSPlatform.Linux);

			// Update all the executable permissions
			const uint ExecutableBits = (1 << 0) | (1 << 3) | (1 << 6);
			foreach(DependencyFile File in Files)
			{
				if(File.IsExecutable)
				{
					string FileName = Path.Combine(RootDir, File.Name);
					int StatResult = -1;
					if (IsLinux)
					{
						StatResult = GetFileMode_Linux(FileName);
					}
					else
					{
						StatResult = GetFileMode_Mac(FileName);
					}

					if (StatResult == -1)
					{
						Log.WriteError($"Stat() call for {File.Name} failed: errorcode: {Marshal.GetLastSystemError()}");
						return false;
					}

					// Get the current permissions
					uint CurrentPermissions = (uint)StatResult;

					// The desired permissions should be executable for every read group
					uint NewPermissions = CurrentPermissions | ((CurrentPermissions >> 2) & ExecutableBits);

					// Update them if they don't match
					if (CurrentPermissions != NewPermissions)
					{
						int ChmodResult = -1;
						if (IsLinux)
						{
							ChmodResult = SetFileMode_Linux(FileName, (ushort)NewPermissions);
						}
						else
						{
							ChmodResult = SetFileMode_Mac(FileName, (ushort)NewPermissions);
						}

						if (ChmodResult != 0)
						{
							Log.WriteError("Chmod() call for {0} failed with error {1}", File.Name, ChmodResult);
							return false;
						}
					}
				}
			}
			return true;
		}

		static bool IsExcludedFolder(string Name, IEnumerable<string> ExcludeFolders)
		{
			string RootedName = "/" + Name;
			foreach(string ExcludeFolder in ExcludeFolders)
			{
				if(ExcludeFolder.StartsWith("/"))
				{
					// Do a prefix check
					if(RootedName.StartsWith(ExcludeFolder + "/", StringComparison.CurrentCultureIgnoreCase))
					{
						return true;
					}
				}
				else
				{
					// Do a substring check
					if(RootedName.IndexOf("/" + ExcludeFolder + "/", StringComparison.CurrentCultureIgnoreCase) != -1)
					{
						return true;
					}
				}
			}
			return false;
		}

		static bool DownloadDependencies(string RootPath, IEnumerable<DependencyFile> RequiredFiles, IEnumerable<DependencyBlob> Blobs, IEnumerable<DependencyPackInfo> Packs, int NumThreads, double HttpTimeoutMultiplier, int MaxRetries, Uri? Proxy, string? CachePath)
		{

			// Build a lookup for the files that need updating from each blob
			Dictionary<string, List<DependencyFile>> BlobToFiles = new Dictionary<string,List<DependencyFile>>();
			foreach(DependencyFile RequiredFile in RequiredFiles)
			{
				if(!BlobToFiles.TryGetValue(RequiredFile.Hash, out List<DependencyFile>? FileList))
				{
					FileList = new List<DependencyFile>();
					BlobToFiles.Add(RequiredFile.Hash, FileList);
				}
				FileList.Add(RequiredFile);
			}

			// Find all the required blobs
			DependencyBlob[] RequiredBlobs = Blobs.Where(x => BlobToFiles.ContainsKey(x.Hash)).ToArray();

			// Build a lookup for the files that need updating from each blob
			Dictionary<string, List<DependencyBlob>> PackToBlobs = new Dictionary<string,List<DependencyBlob>>();
			foreach(DependencyBlob RequiredBlob in RequiredBlobs)
			{
				if(!PackToBlobs.TryGetValue(RequiredBlob.PackHash, out List<DependencyBlob>? BlobList))
				{
					BlobList = new List<DependencyBlob>();
					PackToBlobs.Add(RequiredBlob.PackHash, BlobList);
				}
				BlobList.Add(RequiredBlob);
			}

			// Find all the required packs
			DependencyPackInfo[] RequiredPacks = Packs.Where(x => PackToBlobs.ContainsKey(x.Pack.Hash)).ToArray();

			// Create the download queue
			ConcurrentQueue<IncomingPack> DownloadQueue = new ConcurrentQueue<IncomingPack>();
			foreach(DependencyPackInfo RequiredPack in RequiredPacks)
			{
				IncomingPack Pack = new IncomingPack
				{
					Url = $"{RequiredPack.Manifest.BaseUrl}/{RequiredPack.Pack.RemotePath}/{RequiredPack.Pack.Hash}",
					Hash = RequiredPack.Pack.Hash,
					CacheFileName = (CachePath == null) ? null : Path.Combine(CachePath, RequiredPack.GetCacheFileName()),
					Files = GetIncomingFilesForPack(RootPath, RequiredPack.Pack, PackToBlobs, BlobToFiles),
					CompressedSize = RequiredPack.Pack.CompressedSize
				};
				DownloadQueue.Enqueue(Pack);
			}

			// Setup the async state
			AsyncDownloadState State = new AsyncDownloadState
			{
				NumFiles = RequiredFiles.Count(),
				NumBytesTotal = RequiredPacks.Sum(x => x.Pack.CompressedSize)
			};

			// Setup the http connection object..
			CreateHttpClient(Proxy, RequiredPacks.Max(x => x.Pack.CompressedSize), HttpTimeoutMultiplier, NumThreads);

			// Create all the worker threads
			CancellationTokenSource CancellationToken = new CancellationTokenSource();
			Thread[] WorkerThreads = new Thread[NumThreads];
			for(int Idx = 0; Idx < NumThreads; Idx++)
			{
				WorkerThreads[Idx] = new Thread(x => DownloadWorker(DownloadQueue, State, MaxRetries, CancellationToken.Token));
				WorkerThreads[Idx].Start();
			}

			// Tick the status message until we've finished or ended with an error. Use a circular buffer to average out the speed over time.
			long[] NumBytesReadBuffer = new long[60];
			for (int BufferIdx = 0, NumFilesReportedRead = 0; NumFilesReportedRead < State.NumFiles && State.NumFailingOrIdleDownloads < NumThreads; BufferIdx = (BufferIdx + 1) % NumBytesReadBuffer.Length)
			{
				const int TickInterval = 100;
				Thread.Sleep(TickInterval);

				long NumBytesRead = Interlocked.Read(ref State.NumBytesRead);
				long NumBytesTotal = Interlocked.Read(ref State.NumBytesTotal);
				long NumBytesCached = Interlocked.Read(ref State.NumBytesCached);
				long NumBytesPerSecond = (long)Math.Ceiling((float)Math.Max(NumBytesRead - NumBytesReadBuffer[BufferIdx], 0) * 1000.0f / (NumBytesReadBuffer.Length * TickInterval));

				NumFilesReportedRead = State.NumFilesRead;
				NumBytesReadBuffer[BufferIdx] = NumBytesRead;

				StringBuilder Status = new StringBuilder();
				Status.AppendFormat("Updating dependencies: {0,3}% ({1}/{2})", ((NumBytesRead + NumBytesCached) * 100) / (NumBytesTotal + NumBytesCached), NumFilesReportedRead, State.NumFiles);
				if(NumBytesRead > 0)
				{
					Status.AppendFormat(", {0}/{1} MiB | {2} MiB/s", FormatMegabytes(NumBytesRead, 1), FormatMegabytes(NumBytesTotal, 1), FormatMegabytes(NumBytesPerSecond, 2));
				}
				if(NumBytesCached > 0)
				{
					Status.AppendFormat(", {0} MiB cached", FormatMegabytes(NumBytesCached, 1));
				}
				Status.Append((NumFilesReportedRead == State.NumFiles)? ", done." : "...");
				Log.WriteStatus(Status.ToString());
			}
			Log.FlushStatus();

			// If we finished with an error, try to clean up and return
			if(State.NumFilesRead < State.NumFiles)
			{
				CancellationToken.Cancel();

				if(State.LastDownloadError != null)
				{
					Log.WriteError("{0}", State.LastDownloadError);
				}
				else
				{
					Log.WriteError("Aborting dependency updating due to unknown failure(s).");
				}
				return false;
			}
			else
			{
				foreach(Thread WorkerThread in WorkerThreads)
				{
					WorkerThread.Join();
				}
				return true;
			}
		}

		static string FormatMegabytes(long Value, int NumDecimalPlaces)
		{
			int Multiplier = (int)Math.Pow(10.0, NumDecimalPlaces);
			long FormatValue = ((Value * Multiplier) + (1024 * 1024) - 1) / (1024 * 1024);
			string Result = String.Format("{0}.{1:D" + NumDecimalPlaces.ToString() + "}", FormatValue / Multiplier, FormatValue % Multiplier);
			return Result;
		}

		static IncomingFile[] GetIncomingFilesForPack(string RootPath, DependencyPack RequiredPack, Dictionary<string, List<DependencyBlob>> PackToBlobs, Dictionary<string, List<DependencyFile>> BlobToFiles)
		{
			List<IncomingFile> Files = new List<IncomingFile>();
			foreach(DependencyBlob RequiredBlob in PackToBlobs[RequiredPack.Hash])
			{
				IncomingFile File = new IncomingFile
				{
					Names = BlobToFiles[RequiredBlob.Hash].Select(x => Path.Combine(RootPath, x.Name)).ToArray(),
					Hash = RequiredBlob.Hash,
					MinPackOffset = RequiredBlob.PackOffset,
					MaxPackOffset = RequiredBlob.PackOffset + RequiredBlob.Size
				};
				Files.Add(File);
			}
			return Files.OrderBy(x => x.MinPackOffset).ToArray();
		}

		static void DownloadWorker(ConcurrentQueue<IncomingPack> DownloadQueue, AsyncDownloadState State, int MaxRetries, CancellationToken CancellationToken)
		{
			int Retries = 0;
			for(;;)
			{
				if (CancellationToken.IsCancellationRequested)
				{
					Interlocked.Increment(ref State.NumFailingOrIdleDownloads);
					return;
				}

				// Remove the next file from the download queue, or wait before polling again
				if (!DownloadQueue.TryDequeue(out IncomingPack? NextPack))
				{
					Interlocked.Increment(ref State.NumFailingOrIdleDownloads);
					while(State.NumFilesRead < State.NumFiles && !DownloadQueue.TryDequeue(out NextPack))
					{
						Thread.Sleep(100);
					}
					Interlocked.Decrement(ref State.NumFailingOrIdleDownloads);
				}

				// Quit if we exited the loop because we're finished
				if(NextPack == null)
				{
					break;
				}

				// Try to download the file
				long RollbackSize = 0;
				try
				{
					// Download the pack file or extract it from the cache
					if (TryUnpackFromCache(NextPack.CacheFileName, NextPack.Files))
					{
						Interlocked.Add(ref State.NumBytesCached, NextPack.CompressedSize);
					}
					else
					{
						DownloadAndExtractFiles(NextPack.Url, NextPack.CacheFileName, NextPack.CompressedSize, NextPack.Hash, NextPack.Files, Size => { RollbackSize += Size; Interlocked.Add(ref State.NumBytesRead, Size); }).Wait();
					}

					// Update the stats
					Interlocked.Add(ref State.NumBytesTotal, RollbackSize - NextPack.CompressedSize);
					Interlocked.Add(ref State.NumFilesRead, NextPack.Files.Sum(x => x.Names.Length));

					// If we were failing, decrement the number of failing threads
					if(Retries > MaxRetries)
					{
						Interlocked.Decrement(ref State.NumFailingOrIdleDownloads);
						Retries = 0;
					}
				}
				catch(Exception Ex)
				{
					// Rollback the byte count and add the file back into the download queue
					Interlocked.Add(ref State.NumBytesRead, -RollbackSize);
					DownloadQueue.Enqueue(NextPack);

					// If we've retried enough times already, set the error message. 
					if (Retries++ == MaxRetries)
					{
						Interlocked.Increment(ref State.NumFailingOrIdleDownloads);
						State.LastDownloadError = $"Failed to download '{NextPack.Url}': {FormatExceptionDetails(Ex)}";
					}
				}
			}
			if (Retries < MaxRetries)
			{
				Interlocked.Increment(ref State.NumFailingOrIdleDownloads);
			}
		}

		static bool TryUnpackFromCache(string? CacheFileName, IncomingFile[] Files)
		{
			if (CacheFileName != null && File.Exists(CacheFileName))
			{
				// Try to open the cached file for reading. Could fail due to race conditions despite checking above, so swallow any exceptions.
				FileStream InputStream;
				try
				{
					InputStream = File.Open(CacheFileName, FileMode.Open, FileAccess.Read, FileShare.Read | FileShare.Delete);
				}
				catch(Exception)
				{
					return false;
				}

				// Try to extract files from the cache. If we get a corrupt pack file exception, delete it.
				try
				{
					ExtractFiles(InputStream, Files);
					return true;
				}
				catch(CorruptPackFileException)
				{
					SafeDeleteFileQuiet(CacheFileName);
				}
				finally
				{
					InputStream.Dispose();
				}
			}
			return false;
		}

		static async Task DownloadAndExtractFiles(string Url, string? CacheFileName, long CompressedSize, string ExpectedHash, IncomingFile[] Files, NotifyReadDelegate NotifyRead)
		{
			// Read the response and extract the files
			using(HttpResponseMessage Response = await HttpClientInstance!.GetAsync(Url))
			{
				using(Stream ResponseStream = new NotifyReadStream(Response.Content.ReadAsStream(), NotifyRead))
				{
					if(CacheFileName == null)
					{
						ExtractFiles(ResponseStream, Files);
					}
					else
					{
						ExtractFilesThroughCache(ResponseStream, CacheFileName, CompressedSize, ExpectedHash, Files);
					}
				}
			}
		}

		static void CreateHttpClient(Uri? Proxy, long LargestPackSize, double HttpTimeoutMultiplier, int NumThreads)
		{
			// Create the httpclient using a proxy if needed, and setting connectiosn based on threads.
			HttpClientHandler Handler = new HttpClientHandler
			{
				MaxConnectionsPerServer = NumThreads
			};

			if (Proxy != null)
			{
				Handler.Proxy = new WebProxy(Proxy, true, null, MakeCredentialsFromUri(Proxy));
			}

			HttpClientInstance = new HttpClient(Handler);

			// Estimate and set HttpClient timeout based on the largest pack size
			double EstimatedDownloadDurationSecondsAt2MBps = Convert.ToDouble(LargestPackSize) / 250000.0;
			TimeSpan HttpTimeout = TimeSpan.FromSeconds(Math.Max(HttpClientInstance.Timeout.TotalSeconds, EstimatedDownloadDurationSecondsAt2MBps * HttpTimeoutMultiplier));
			HttpClientInstance.Timeout = HttpTimeout;
		}

		static NetworkCredential? MakeCredentialsFromUri(Uri Address)
		{
			// Check if the URI has a login:password prefix, and convert it to a NetworkCredential object if it has. HttpRequest just ignores it.
			if(!String.IsNullOrEmpty(Address.UserInfo))
			{
				int Index = Address.UserInfo.IndexOf(':');
				if(Index != -1)
				{
					return new NetworkCredential(Address.UserInfo.Substring(0, Index), Address.UserInfo.Substring(Index + 1));
				}
			}
			return null;
		}

		static void ExtractFiles(Stream InputStream, IncomingFile[] Files)
		{
			// Create a decompression stream around the raw input stream
			GZipStream DecompressedStream = new GZipStream(InputStream, CompressionMode.Decompress, true);
			ExtractFilesFromRawStream(DecompressedStream, Files, null);
		}

		static void ExtractFilesThroughCache(Stream InputStream, string FileName, long CompressedSize, string ExpectedHash, IncomingFile[] Files)
		{
			// Extract files from a pack file while writing to the cache file at the same time
			string IncomingFileName = $"{FileName}-{InstanceSuffix}{IncomingFileSuffix}";
			try
			{
				// Make sure the directory exists
				Directory.CreateDirectory(Path.GetDirectoryName(IncomingFileName)!);

				// Hash the uncompressed data as we go
				SHA1 Hasher = SHA1.Create();
				using(FileStream CacheStream = File.Open(IncomingFileName, FileMode.Create, FileAccess.Write, FileShare.None))
				{
					CacheStream.SetLength(CompressedSize);

					ForkReadStream ForkedInputStream = new ForkReadStream(InputStream, CacheStream);
					using(GZipStream DecompressedStream = new GZipStream(ForkedInputStream, CompressionMode.Decompress, true))
					{
						ExtractFilesFromRawStream(DecompressedStream, Files, Hasher);
					}
				}

				// Check the hash was what we expected, and move it into the cache if it is.
				string Hash = BitConverter.ToString(Hasher.Hash!).ToLower().Replace("-", "");
				if (Hash != ExpectedHash)
				{
					throw new CorruptPackFileException(String.Format("Incorrect hash for pack - expected {0}, got {1}", ExpectedHash, Hash), null);
				}

				// Move the new cache file into place
				SafeMoveFileQuiet(IncomingFileName, FileName);
			}
			finally
			{
				SafeDeleteFileQuiet(IncomingFileName);
			}
		}

		static void ExtractFilesFromRawStream(Stream RawStream, IncomingFile[] Files, SHA1? RawStreamHasher)
		{
			int MinFileIdx = 0;
			int MaxFileIdx = 0;
			FileStream[] OutputStreams = new FileStream[Files.Length];
			SHA1[] OutputHashers = new SHA1[Files.Length];
			try
			{
				// Create files from pack.
				byte[] Buffer = new byte[16384];
				long PackOffset = 0;
				while(MinFileIdx < Files.Length || RawStreamHasher != null)
				{
					// Read the next chunk of data
					int ReadSize;
					try
					{
						ReadSize = RawStream.Read(Buffer, 0, Buffer.Length);
					}
					catch (Exception Ex)
					{
						throw new CorruptPackFileException("Can't read from pack stream", Ex);
					}
					if (ReadSize == 0)
					{
						break;
					}

					// Transform the raw stream hash
					if(RawStreamHasher != null)
					{
						RawStreamHasher.TransformBlock(Buffer, 0, ReadSize, Buffer, 0);
					}

					// Write to all the active files
					for(int Idx = MinFileIdx; Idx < Files.Length && Files[Idx].MinPackOffset <= PackOffset + ReadSize; Idx++)
					{
						IncomingFile CurrentFile = Files[Idx];

						// Open the stream if it's a new file
						if(Idx == MaxFileIdx)
						{
							Directory.CreateDirectory(Path.GetDirectoryName(CurrentFile.Names[0])!);
							OutputStreams[Idx] = File.Open(CurrentFile.Names[0] + IncomingFileSuffix, FileMode.Create, FileAccess.Write, FileShare.None);
							OutputStreams[Idx].SetLength(CurrentFile.MaxPackOffset - CurrentFile.MinPackOffset);
							OutputHashers[Idx] = SHA1.Create();
							MaxFileIdx++;
						}

						// Write the data to this file
						int BufferOffset = (int)Math.Max(0, CurrentFile.MinPackOffset - PackOffset);
						int BufferCount = (int)Math.Min(ReadSize, CurrentFile.MaxPackOffset - PackOffset) - BufferOffset;
						OutputStreams[Idx].Write(Buffer, BufferOffset, BufferCount);
						OutputHashers[Idx].TransformBlock(Buffer, BufferOffset, BufferCount, Buffer, BufferOffset);

						// If we're finished, verify the hash and close it
						if(Idx == MinFileIdx && CurrentFile.MaxPackOffset <= PackOffset + ReadSize)
						{
							OutputHashers[Idx].TransformFinalBlock(Buffer, 0, 0);

							string Hash = BitConverter.ToString(OutputHashers[Idx].Hash!).ToLower().Replace("-", "");
							if(Hash != CurrentFile.Hash)
							{
								throw new CorruptPackFileException(String.Format("Incorrect hash value of {0}: expected {1}, got {2}", CurrentFile.Names[0], CurrentFile.Hash, Hash), null);
							}

							OutputStreams[Idx].Dispose();

							for(int FileIdx = 1; FileIdx < CurrentFile.Names.Length; FileIdx++)
							{
								Directory.CreateDirectory(Path.GetDirectoryName(CurrentFile.Names[FileIdx])!);
								File.Copy(CurrentFile.Names[0] + IncomingFileSuffix, CurrentFile.Names[FileIdx] + IncomingFileSuffix, true);
								File.Delete(CurrentFile.Names[FileIdx]);
								File.Move(CurrentFile.Names[FileIdx] + IncomingFileSuffix, CurrentFile.Names[FileIdx]);
							}

							File.Delete(CurrentFile.Names[0]);
							File.Move(CurrentFile.Names[0] + IncomingFileSuffix, CurrentFile.Names[0]);
							MinFileIdx++;
						}
					}
					PackOffset += ReadSize;
				}

				// If we didn't extract everything, throw an exception
				if(MinFileIdx < Files.Length)
				{
					throw new CorruptPackFileException("Unexpected end of file", null);
				}

				// Transform the final block
				if(RawStreamHasher != null)
				{
					RawStreamHasher.TransformFinalBlock(Buffer, 0, 0);
				}
			}
			finally 
			{
				// Delete unfinished files.
				for(int Idx = MinFileIdx; Idx < MaxFileIdx; Idx++)
				{
					OutputStreams[Idx].Dispose();
					foreach(string Name in Files[Idx].Names)
					{
						SafeDeleteFileQuiet(Name + IncomingFileSuffix);
					}
				}
			}
		}

		static bool ReadXmlObject<T>(string FileName, out T NewObject)
		{
			try
			{
				XmlSerializer Serializer = new XmlSerializer(typeof(T));
				using(StreamReader Reader = new StreamReader(FileName))
				{
					NewObject = (T)Serializer.Deserialize(Reader)!;
				}
				return true;
			}
			catch(Exception Ex)
			{
				Log.WriteError($"Failed to read '{FileName}': {FormatExceptionDetails(Ex)}");
				NewObject = default(T)!;
				return false;
			}
		}

		static bool WriteXmlObject<T>(string FileName, T XmlObject)
		{
			try
			{
				XmlSerializer Serializer = new XmlSerializer(typeof(T));
				using(StreamWriter Writer = new StreamWriter(FileName))
				{
					XmlWriterSettings WriterSettings = new() { Indent = true };
					using(XmlWriter XMLWriter = XmlWriter.Create(Writer, WriterSettings))
					{
						Serializer.Serialize(XMLWriter, XmlObject);
					}
				}
				return true;
			}
			catch(Exception Ex)
			{
				Log.WriteError($"Failed to write file '{FileName}': {FormatExceptionDetails(Ex)}");
				return false;
			}
		}

		static bool WriteWorkingManifest(string FileName, string TemporaryFileName, WorkingManifest Manifest)
		{
			if(!WriteXmlObject(TemporaryFileName, Manifest))
			{
				return false;
			}
			if(!SafeModifyFileAttributes(TemporaryFileName, FileAttributes.Hidden, 0))
			{
				return false;
			}
			if(!SafeDeleteFile(FileName))
			{
				return false;
			}
			if(!SafeMoveFile(TemporaryFileName, FileName))
			{
				return false;
			}
			return true;
		}

		static bool SafeModifyFileAttributes(string FileName, FileAttributes AddAttributes, FileAttributes RemoveAttributes)
		{
			try
			{
				File.SetAttributes(FileName, (File.GetAttributes(FileName) | AddAttributes) & ~RemoveAttributes);
				return true;
			}
			catch(IOException)
			{
				Log.WriteError("Failed to set attributes for file '{0}'", FileName);
				return false;
			}
		}

		static bool SafeDeleteFile(string FileName)
		{
			try
			{
				File.Delete(FileName);
				return true;
			}
			catch(IOException)
			{
				Log.WriteError("Failed to delete file '{0}'", FileName);
				return false;
			}
		}

		static bool SafeDeleteFileQuiet(string FileName)
		{
			try
			{
				File.Delete(FileName);
				return true;
			}
			catch(IOException)
			{
				return false;
			}
		}

		static bool SafeMoveFile(string SourceFileName, string TargetFileName)
		{
			try
			{
				File.Move(SourceFileName, TargetFileName);
				return true;
			}
			catch(IOException)
			{
				Log.WriteError("Failed to rename '{0}' to '{1}'", SourceFileName, TargetFileName);
				return false;
			}
		}

		static bool SafeMoveFileQuiet(string SourceFileName, string TargetFileName)
		{
			try
			{
				File.Move(SourceFileName, TargetFileName);
				return true;
			}
			catch(IOException)
			{
				return false;
			}
		}

		static string ComputeHashForFile(string FileName)
		{
			SHA1 Hasher = SHA1.Create();
			using(FileStream InputStream = File.OpenRead(FileName))
			{
				byte[] Hash = Hasher.ComputeHash(InputStream);
				return BitConverter.ToString(Hash).ToLower().Replace("-", "");
			}
		}

		static void BuildExceptionStack(Exception? exception, List<Exception> exceptionStack)
		{
			if (exception != null)
			{
				exceptionStack.Add(exception);

				AggregateException? aggregateException = exception as AggregateException;
				if (aggregateException != null && aggregateException.InnerExceptions.Count > 0)
				{
					for(int idx = 0; idx < 16 && idx < aggregateException.InnerExceptions.Count; idx++) // Cap number of exceptions returned to avoid huge messages
					{
						BuildExceptionStack(aggregateException.InnerExceptions[idx], exceptionStack);
					}
				}
				else
				{
					if (exception.InnerException != null)
					{
						BuildExceptionStack(exception.InnerException, exceptionStack);
					}
				}
			}
		}

		static string FormatExceptionDetails(Exception ex)
		{
			List<Exception> exceptionStack = new List<Exception>();
			BuildExceptionStack(ex, exceptionStack);

			StringBuilder message = new StringBuilder();
			for (int idx = exceptionStack.Count - 1; idx >= 0; idx--)
			{
				Exception currentEx = exceptionStack[idx];
				message.AppendFormat("{0}{1}: {2}\n{3}", (idx == exceptionStack.Count - 1) ? "" : "Wrapped by ", currentEx.GetType().Name, currentEx.Message, currentEx.StackTrace);

				if (currentEx.Data.Count > 0)
				{
					foreach (object key in currentEx.Data.Keys)
					{
						if (key == null)
						{
							continue;
						}

						object? value = currentEx.Data[key];
						if (value == null)
						{
							continue;
						}

						string valueString;
						if (value is List<string> valueList)
						{
							valueString = String.Format("({0})", String.Join(", ", valueList.Select(x => String.Format("\"{0}\"", x))));
						}
						else
						{
							valueString = value.ToString() ?? String.Empty;
						}

						message.AppendFormat("   data: {0} = {1}", key, valueString);
					}
				}
			}
			return message.Replace("\r\n", "\n").ToString();
		}
	}
}
