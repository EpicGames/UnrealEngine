@echo off
setlocal
pushd "%~dp0"
call "..\..\..\Build\BatchFiles\GetDotnetPath.bat"
if exist "..\..\..\Binaries\DotNET\GitDependencies\*" (
	del /s /q "..\..\..\Binaries\DotNET\GitDependencies\*"
)

echo.
echo Building for win-x64...
if exist bin ( rmdir /s /q bin )
if exist obj ( rmdir /s /q obj )
"%DOTNET_ROOT%\dotnet" publish GitDependencies.csproj -r win-x64 -c Release --output "..\..\..\Binaries\DotNET\GitDependencies\win-x64" --nologo --self-contained
if errorlevel 1 goto :eof

echo.
echo Building for linux-x64...
rmdir /s /q bin
rmdir /s /q obj
"%DOTNET_ROOT%\dotnet" publish GitDependencies.csproj -r linux-x64 -c Release --output "..\..\..\Binaries\DotNET\GitDependencies\linux-x64" --nologo --self-contained
if errorlevel 1 goto :eof

endlocal
