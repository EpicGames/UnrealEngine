#!/bin/sh
# Copyright Epic Games, Inc. All Rights Reserved.
# Pass -Sign to sign the executables, disabled by default

set -e

SCRIPT_PATH=$0
if [ -L "$SCRIPT_PATH" ]; then
	SCRIPT_PATH="$(dirname "$SCRIPT_PATH")/$(readlink "$SCRIPT_PATH")"
fi

cd "$(dirname "$SCRIPT_PATH")" && SCRIPT_PATH="`pwd`/$(basename "$SCRIPT_PATH")"

pushd "$(dirname "$SCRIPT_PATH")" > /dev/null
sh ../../../Build/BatchFiles/Mac/SetupDotnet.sh
rm -R -f "../../../Binaries/DotNET/GitDependencies/"

echo
echo Building for osx-x64...
rm -R -f bin
rm -R -f obj
dotnet publish GitDependencies.csproj -r osx-x64 -c Release --output "../../../Binaries/DotNET/GitDependencies/osx-x64" --nologo --self-contained
if [ $? -ne 0 ]; then
	echo GitDependencies: Failed to  build for osx-x64
	exit 1
fi

echo
echo Building for osx-arm64...
rm -R -f bin
rm -R -f obj
dotnet publish GitDependencies.csproj -r osx-arm64 -c Release --output "../../../Binaries/DotNET/GitDependencies/osx-arm64" --nologo --self-contained
if [ $? -ne 0 ]; then
	echo GitDependencies: Failed to  build for osx-arm64
	exit 1
fi

# Sign the GitDependency binaries
if [[ "$*" == *"-Sign"* ]]; then
	/usr/bin/codesign -f -s "Developer ID Application" -v "../../../Binaries/DotNET/GitDependencies/osx-x64/GitDependencies" --no-strict
	/usr/bin/codesign -f -s "Developer ID Application" -v "../../../Binaries/DotNET/GitDependencies/osx-arm64/GitDependencies" --no-strict
fi

popd
