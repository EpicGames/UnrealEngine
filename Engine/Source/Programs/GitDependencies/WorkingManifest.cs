// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Xml.Serialization;

namespace GitDependencies
{
	public class WorkingFile
	{
		[XmlAttribute]
		public string Name = null!;

		[XmlAttribute]
		public string Hash = null!;

		[XmlAttribute]
		public string ExpectedHash = null!;

		[XmlAttribute]
		public long Timestamp;
	}

	public class WorkingManifest
	{
		[XmlArrayItem("File")]
		public List<WorkingFile> Files = new List<WorkingFile>();
	}
}
